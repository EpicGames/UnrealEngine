[DerivedDataCacheGraphs]
Default=(ProjectPak, EnginePak, ZenLocal, Local, ZenShared, Cloud)
DerivedDataBackendGraph=(ProjectPak, EnginePak, ZenLocal, Local, ZenShared, Cloud)

[DerivedDataCacheStores]
Local=(Type=FileSystem, ReadOnly=false, Clean=false, Flush=false, Path=../../../Engine/Programs/%GAME%/DDC)
ProjectPak=(Type=ReadPak, Filename=../../../Engine/Programs/%GAME%/DDC/DDC.ddp)
