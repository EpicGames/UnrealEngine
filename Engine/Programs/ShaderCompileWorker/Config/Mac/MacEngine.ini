[DerivedDataCacheStores]
Shared=(Type=FileSystem, ReadOnly=false, Clean=false, Flush=false, DeleteUnused=false, UnusedFileAge=38, FoldersToClean=-1, Path=/Volumes/UE4DDC, EnvPathOverride=UE-SharedDataCachePath)
